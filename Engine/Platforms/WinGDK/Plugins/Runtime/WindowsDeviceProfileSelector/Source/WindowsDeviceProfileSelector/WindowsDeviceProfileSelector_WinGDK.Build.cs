// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WindowsDeviceProfileSelector_WinGDK : WindowsDeviceProfileSelector
	{
		public WindowsDeviceProfileSelector_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			// use the platform name from the underlying windows properties - This will be Windows, WindowsClient etc.
			// this must match FWinGDKTargetPlatform::CookingDeviceProfileName()
			PrivateDefinitions.Add($"WINDOWS_OVERRIDE_DEVICEPROFILE_NAME=FPlatformProperties::TBaseWindowsPlatformProperties::PlatformName()");
		}
	}
}
