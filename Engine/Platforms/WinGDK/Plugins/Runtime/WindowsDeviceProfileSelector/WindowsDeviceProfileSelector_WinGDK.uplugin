{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"Modules": [
		{
			"Name": "WindowsDeviceProfileSelector",
			"Type": "RuntimeNoCommandlet",
			"PlatformAllowList": [
				"WinGDK"
			]
		}
	]
}