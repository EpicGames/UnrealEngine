// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsDeviceProfileSelectorModule.h"
#include "Modules/ModuleManager.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "DynamicRHI.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Internationalization/Regex.h"

IMPLEMENT_MODULE(FWindowsDeviceProfileSelectorModule, WindowsDeviceProfileSelector);


static TSet<FString> GetRHIDeviceProfileNames(const FString& BaseProfileName)
{
	// BaseProfileName is embedded directly into a regex pattern, so it must not contain
	// any regex special characters. Profile names are always plain platform name strings
	// (e.g. "Windows", "WindowsEditor"), so this would indicate a programming error.
	for (TCHAR C : BaseProfileName)
	{
		checkf(FChar::IsAlnum(C) || C == TCHAR('_'),
			TEXT("BaseProfileName '%s' contains unexpected character '%c'"), *BaseProfileName, C);
	}

	TSet<FString> RHIProfileNames;
	const FConfigFile* DeviceProfilesConfig = GConfig->Find(GDeviceProfilesIni);
	if (DeviceProfilesConfig == nullptr)
	{
		return RHIProfileNames;
	}

	const FRegexPattern RHIProfilePattern(
		FString::Printf(TEXT("^%s_(\\w+) DeviceProfile$"), *BaseProfileName));
	const int32 DeviceProfileSuffixLen = FString(TEXT(" DeviceProfile")).Len();
	for (const auto& [SectionName, Section] : *DeviceProfilesConfig)
	{
		FRegexMatcher Matcher(RHIProfilePattern, SectionName);
		if (Matcher.FindNext())
		{
			RHIProfileNames.Add(SectionName.Left(SectionName.Len() - DeviceProfileSuffixLen));
		}
	}
	return RHIProfileNames;
}


void FWindowsDeviceProfileSelectorModule::StartupModule()
{
}


void FWindowsDeviceProfileSelectorModule::ShutdownModule()
{
}

const FString FWindowsDeviceProfileSelectorModule::GetRuntimeDeviceProfileName()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWindowsDeviceProfileSelectorModule::GetRuntimeDeviceProfileName);

	static FString ProfileName;

	if (ProfileName.IsEmpty())
	{
#if defined(WINDOWS_OVERRIDE_DEVICEPROFILE_NAME)
		// overridden device profile name, typically for Windows-based platform extensions
		ProfileName = WINDOWS_OVERRIDE_DEVICEPROFILE_NAME;
#elif UE_IS_COOKED_EDITOR
		// some heuristics to determine a cooked editor
		ProfileName = TEXT("WindowsCookedEditor");
#else
		// Windows, WindowsEditor, WindowsClient, or WindowsServer
		ProfileName = FPlatformProperties::PlatformName();
#endif

		if (FApp::CanEverRender())
		{
			// Scan all section names to find RHI-specific device profiles for this base profile name.
			// This avoids calling the expensive GetSelectedDynamicRHIModuleName when no such profiles exist.
			TSet<FString> ConfigRHIProfileNames = GetRHIDeviceProfileNames(ProfileName);

			if (ConfigRHIProfileNames.Num() > 0)
			{
				FString TmpProfileName = ProfileName + TCHAR('_') + GetSelectedDynamicRHIModuleName(false);
				if (ConfigRHIProfileNames.Contains(TmpProfileName))
				{
					// Use RHI specific device profile if it exists
					ProfileName = MoveTemp(TmpProfileName);
				}
				else if (TmpProfileName.Contains(TEXT("_ES31")))
				{
					TmpProfileName = ProfileName + TEXT("_ES31");
					if (ConfigRHIProfileNames.Contains(TmpProfileName))
					{
						// General ES31 device profile
						ProfileName = MoveTemp(TmpProfileName);
					}
				}
			}
		}
	}
	
	// Use override from ConfigRules if set
	const FString* ConfigProfile = FPlatformMisc::GetConfigRulesVariable(TEXT("Profile"));
	if (ConfigProfile != nullptr)
	{
		ProfileName = *ConfigProfile;
		UE_LOGF(LogWindows, Log, "Using ConfigRules Profile: [%ls]", *ProfileName);
	}

	return ProfileName;
}
