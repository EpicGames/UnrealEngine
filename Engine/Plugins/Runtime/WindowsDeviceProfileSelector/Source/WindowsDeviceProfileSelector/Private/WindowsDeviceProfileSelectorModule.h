// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDeviceProfileSelectorModule.h"

/**
 * Implements the Windows Device Profile Selector module.
 */
class FWindowsDeviceProfileSelectorModule
	: public IDeviceProfileSelectorModule
{
public:

	//~ Begin IDeviceProfileSelectorModule Interface
	virtual const FString GetRuntimeDeviceProfileName() override;
	//~ End IDeviceProfileSelectorModule Interface


	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	
	/**
	 * Virtual destructor.
	 */
	virtual ~FWindowsDeviceProfileSelectorModule() = default;
};
