// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class WindowsDeviceProfileSelector : ModuleRules
	{
        public WindowsDeviceProfileSelector(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
				    "Engine",
                    "RHI",
				}
				);
		}
	}
}
