{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Windows Device Profile Selector",
	"Description" : "Windows Device Profile Selector used to determine the system settings for windows platforms",
	"Category" : "Device Profile Selectors",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "WindowsDeviceProfileSelector",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostConfigInit",
			"PlatformAllowList" :
			[
				"Win64"
			]
		}
	]
}